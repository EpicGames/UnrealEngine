[CommonSettings]
SourcePath=Plugins/Runtime/HTTPChunkInstaller/Content/Localization/HTTPChunkInstaller
DestinationPath=Plugins/Runtime/HTTPChunkInstaller/Content/Localization/HTTPChunkInstaller
ManifestName=HTTPChunkInstaller.manifest
ArchiveName=HTTPChunkInstaller.archive
PortableObjectName=HTTPChunkInstaller.po
ResourceName=HTTPChunkInstaller.locres
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=fr
CulturesToGenerate=de
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=it
CulturesToGenerate=ja
CulturesToGenerate=ko
CulturesToGenerate=ru
CulturesToGenerate=zh-Hans
CulturesToGenerate=ar
CulturesToGenerate=pl
CulturesToGenerate=pt-BR

;Gather text from source code
[GatherTextStep0]
CommandletClass=GatherTextFromSource
SearchDirectoryPaths=Plugins/Runtime/HTTPChunkInstaller/Source/
FileNameFilters=*.cpp
FileNameFilters=*.h
FileNameFilters=*.c
FileNameFilters=*.inl
FileNameFilters=*.mm
FileNameFilters=*.ini
ShouldGatherFromEditorOnlyData=false

;Write Manifest
[GatherTextStep1]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep2]
CommandletClass=GenerateGatherArchive

;Import localized PO files
[GatherTextStep3]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep4]
CommandletClass=GenerateTextLocalizationResource

;Export  PO files
[GatherTextStep5]
CommandletClass=InternationalizationExport
bExportLoc=true