{
	"FileVersion" : 3,
	
	"FriendlyName" : "HTTP Chunk Installer",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Implements a streaming install client",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

    "Modules": [
        {
            "Name": "HTTPChunkInstaller",
            "Type": "Runtime",
            "LoadingPhase" : "PostConfigInit",
			"PlatformDenyList" :
			[
			]
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "HTTPChunkInstaller",
			"LoadingPolicy": "Always"
		}
	]
}
