// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HTTPChunkInstaller : ModuleRules
{
	public HTTPChunkInstaller(ReadOnlyTargetRules Target)
		: base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "BuildPatchServices",
                "Core",
				"ApplicationCore",
                "Engine",
                "HTTP",
                "Json",
                "PakFile",
            }
            );
    }
}
