// Copyright Epic Games, Inc. All Rights Reserved.

#include "HTTPChunkInstaller.h"
#include "Async/AsyncWork.h"
#include "HTTPChunkInstallerLog.h"
#include "LocalTitleFile.h"
#include "Misc/SecureHash.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "HAL/RunnableThread.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "HTTPChunkInstaller"

// helper to grab the installer service
static IBuildPatchServicesModule* GetBuildPatchServices()
{
	static IBuildPatchServicesModule* BuildPatchServices = nullptr;
	if (BuildPatchServices == nullptr)
	{
		BuildPatchServices = &FModuleManager::LoadModuleChecked<IBuildPatchServicesModule>(TEXT("BuildPatchServices"));
	}
	return BuildPatchServices;
}

const FName NAME_SHA1(TEXT("SHA1"));
const FName NAME_SHA256(TEXT("SHA256"));

class FTitleFileHttpAsyncLoadAndVerify :
	public FNonAbandonableTask
{
public:
	/** File data loaded for the async read */
	TArray<uint8> FileData;
	/** Amount of data read from the file to be owned/referenced by the game thread */
	FThreadSafeCounter64* BytesRead;
	/** The original name of the file being read */
	const FString OriginalFileName;
	/** The name of the file being read off of disk */
	const FString FileName;
	/** The hash value the backend said it should have */
	const FString ExpectedHash;
	/** The hash type SHA1 or SHA256 right now */
	const FName HashType;
	/** Whether the hashes matched */
	bool bHashesMatched;

	/** Initializes the variables needed to load and verify the data */
	FTitleFileHttpAsyncLoadAndVerify(const FString& InOriginalFileName, const FString& InFileName, const FString& InExpectedHash, const FName InHashType, FThreadSafeCounter64* InBytesReadCounter) :
		BytesRead(InBytesReadCounter),
		OriginalFileName(InOriginalFileName),
		FileName(InFileName),
		ExpectedHash(InExpectedHash),
		HashType(InHashType),
		bHashesMatched(false)
	{

	}

	/**
	* Loads and hashes the file data. Empties the data if the hash check fails
	*/
	void DoWork()
	{
		// load file from disk
		bool bLoadedFile = false;

		FArchive* Reader = IFileManager::Get().CreateFileReader(*FileName, FILEREAD_Silent);
		if (Reader)
		{
			int64 SizeToRead = Reader->TotalSize();
			FileData.Reset(SizeToRead);
			FileData.AddUninitialized(SizeToRead);

			uint8* FileDataPtr = FileData.GetData();

			static const int64 ChunkSize = 100 * 1024;

			int64 TotalBytesRead = 0;
			while (SizeToRead > 0)
			{
				int64 Val = FMath::Min(SizeToRead, ChunkSize);
				Reader->Serialize(FileDataPtr + TotalBytesRead, Val);
				BytesRead->Add(Val);
				TotalBytesRead += Val;
				SizeToRead -= Val;
			}

			ensure(SizeToRead == 0 && Reader->TotalSize() == TotalBytesRead);
			bLoadedFile = Reader->Close();
			delete Reader;
		}

		// verify hash of file if it exists
		if (bLoadedFile)
		{
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "ReadFile request. Local file read from cache =%ls", *FileName);
			if (HashType == NAME_SHA1)
			{
				bHashesMatched = IsValidSHA1(ExpectedHash, FileData);
			}
			else if (HashType == NAME_SHA256)
			{
				bHashesMatched = IsValidSHA256(ExpectedHash, FileData);
			}
		}
		else
		{
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "Local file (%ls) not cached locally", *FileName);
		}
		if (!bHashesMatched)
		{
			// Empty local that was loaded
			FileData.Empty();
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTitleFileHttpAsyncLoadAndVerify, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	/** Validates that a buffer matches the same signature as was specified */
	bool IsValidSHA1(const FString& Hash, const TArray<uint8>& Source) const
	{
		uint8 LocalHash[20];
		FSHA1::HashBuffer(Source.GetData(), Source.Num(), LocalHash);
		// concatenate 20 bye SHA1 hash to string
		FString LocalHashStr;
		for (int Idx = 0; Idx < 20; Idx++)
		{
			LocalHashStr += FString::Printf(TEXT("%02x"), LocalHash[Idx]);
		}
		return Hash == LocalHashStr;
	}
	bool IsValidSHA256(const FString& Hash, const TArray<uint8>& Source) const
	{
		FSHA256Signature Signature;
		if (FPlatformMisc::GetSHA256Signature(Source.GetData(), Source.Num(), Signature))
		{
			return Signature.ToString() == Hash;
		}
		return false;
	}
};

// helper class for an HTTP Online Title File
class FOnlineTitleFileHttp : public ICloudTitleFile
{
public:
	/**
	* Constructor
	*
	* @param InSubsystem mcp subsystem being used
	*/
	FOnlineTitleFileHttp(const FString& InBaseUrl)
		: EnumerateFilesUrl(TEXT(""))
		, BaseUrl(InBaseUrl)
	{
		GConfig->GetString(TEXT("HTTPOnlineTitleFile"), TEXT("BaseUrl"), BaseUrl, GEngineIni);
		GConfig->GetString(TEXT("HTTPOnlineTitleFile"), TEXT("EnumerateFilesUrl"), EnumerateFilesUrl, GEngineIni);
		bCacheFiles = true;
		bPlatformSupportsSHA256 = false;
	}

	/**
	* Destructor
	*/
	virtual ~FOnlineTitleFileHttp()
	{}

	/**
	* Copies the file data into the specified buffer for the specified file
	*
	* @param FileName the name of the file to read
	* @param FileContents the out buffer to copy the data into
	*
	* @return true if the data was copied, false otherwise
	*/
	virtual bool GetFileContents(const FString& FileName, TArray<uint8>& FileContents) override
	{
		const TArray<FCloudEntry>* FilesPtr = &Files;
		if (FilesPtr != NULL)
		{
			for (TArray<FCloudEntry>::TConstIterator It(*FilesPtr); It; ++It)
			{
				if (It->FileName == FileName)
				{
					FileContents = It->Data;
					return true;
				}
			}
		}
		return false;
	}

	/**
	* Empties the set of downloaded files if possible (no async tasks outstanding)
	*
	* @return true if they could be deleted, false if they could not
	*/
	virtual bool ClearFiles() override
	{
		for (int Idx = 0; Idx < Files.Num(); Idx++)
		{
			if (Files[Idx].AsyncState == ECloudAsyncTaskState::InProgress)
			{
				UE_LOGF(LogHTTPChunkInstaller, Warning, "Cant clear files. Pending file op for %ls", *Files[Idx].FileName);
				return false;
			}
		}
		// remove all cached file entries
		Files.Empty();
		return true;
	}

	/**
	* Empties the cached data for this file if it is not being downloaded currently
	*
	* @param FileName the name of the file to remove from the cache
	*
	* @return true if it could be deleted, false if it could not
	*/
	virtual bool ClearFile(const FString& FileName) override
	{
		for (int Idx = 0; Idx < Files.Num(); Idx++)
		{
			if (Files[Idx].FileName == FileName)
			{
				if (Files[Idx].AsyncState == ECloudAsyncTaskState::InProgress)
				{
					UE_LOGF(LogHTTPChunkInstaller, Warning, "Cant clear file. Pending file op for %ls", *Files[Idx].FileName);
					return false;
				}
				else
				{
					Files.RemoveAt(Idx);
					return true;
				}
			}
		}
		return false;
	}

	/**
	* Delete cached files on disk
	*
	* @param bSkipEnumerated if true then only non-enumerated files are deleted
	*/
	virtual void DeleteCachedFiles(bool bSkipEnumerated) override
	{
		TArray<FString> CachedFiles;
		IFileManager::Get().FindFiles(CachedFiles, *(GetLocalCachePath() / TEXT("*")), true, false);

		for (auto CachedFile : CachedFiles)
		{
			bool bSkip = bSkipEnumerated && GetCloudFileHeader(CachedFile);
			if (!bSkip)
			{
				IFileManager::Get().Delete(*GetLocalFilePath(CachedFile), false, true);
			}
		}
	}

	/**
	* Requests a list of available files from the network store
	*
	* @return true if the request has started, false if not
	*/
	virtual bool EnumerateFiles(const FCloudPagedQuery& Page = FCloudPagedQuery()) override
	{
		FString ErrorStr;
		bool bStarted = false;

		// Make sure an enumeration request  is not currently pending
		if (EnumerateFilesRequests.Num() > 0)
		{
			ErrorStr = TEXT("Request already in progress.");
		}
		else
		{
			// Create the Http request and add to pending request list
			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
			EnumerateFilesRequests.Add(HttpRequest, Page);

			HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineTitleFileHttp::EnumerateFiles_HttpRequestComplete);
			HttpRequest->SetURL(GetBaseUrl()+EnumerateFilesUrl+TEXT("/Master.manifest"));
			HttpRequest->SetVerb(TEXT("GET"));
			bStarted = HttpRequest->ProcessRequest();
		}
		if (!bStarted)
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "EnumerateFiles request failed. %ls", *ErrorStr);
			TriggerOnEnumerateFilesCompleteDelegates(false);
		}
		return bStarted;
	}

	/**
	* Returns the list of files that was returned by the network store
	*
	* @param Files out array of file metadata
	*
	*/
	virtual void GetFileList(TArray<FCloudHeader>& OutFiles) override
	{
		TArray<FCloudHeader>* FilesPtr = &FileHeaders;
		if (FilesPtr != NULL)
		{
			OutFiles = *FilesPtr;
		}
		else
		{
			OutFiles.Empty();
		}
	}

	/**
	* Starts an asynchronous read of the specified file from the network platform's file store
	*
	* @param FileToRead the name of the file to read
	*
	* @return true if the calls starts successfully, false otherwise
	*/
	virtual bool ReadFile(const FString& FileName) override
	{
		bool bStarted = false;

		FCloudHeader* CloudFileHeader = GetCloudFileHeader(FileName);

		// Make sure valid filename was specified3
		if (FileName.IsEmpty() || FileName.Contains(TEXT(" ")))
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "Invalid filename filename=%ls", *FileName);
			TriggerOnReadFileCompleteDelegates(false, FileName);
			return false;
		}

		// Make sure a file request for this file is not currently pending
		for (const auto& Pair : FileRequests)
		{
			if (Pair.Value == FPendingFileRequest(FileName))
			{
				UE_LOGF(LogHTTPChunkInstaller, Verbose, "ReadFileRemote is already in progress for (%ls)", *FileName);
				return true;
			}
		}

		FCloudEntry* CloudFile = GetCloudFile(FileName, true);
		if (CloudFile->AsyncState == ECloudAsyncTaskState::InProgress)
		{
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "ReadFile is already in progress for (%ls)", *FileName);
			return true;
		}

		if (bCacheFiles)
		{
			// Try to read this from the cache if possible
			bStarted = StartReadFileLocal(FileName);
		}
		if (!bStarted)
		{
			// Failed locally (means not on disk) so fetch from server
			bStarted = ReadFileRemote(FileName);
		}

		if (!bStarted || CloudFile->AsyncState == ECloudAsyncTaskState::Failed)
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "ReadFile request failed for file (%ls)", *FileName);
			TriggerOnReadFileCompleteDelegates(false, FileName);
		}
		else if (CloudFile->AsyncState == ECloudAsyncTaskState::Done)
		{
			TriggerOnReadFileCompleteDelegates(true, FileName);
		}
		return bStarted;
	}

	/** Used to check that async tasks have completed and can be completed */
	virtual void Tick(float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineTitleFileHttp_Tick);

		TArray<int32> ItemsToRemove;
		ItemsToRemove.Reserve(AsyncLocalReads.Num());

		// Check for any completed tasks
		for (int32 TaskIdx = 0; TaskIdx < AsyncLocalReads.Num(); TaskIdx++)
		{
			FTitleAsyncReadData& Task = AsyncLocalReads[TaskIdx];
			if (Task.AsyncTask->IsDone())
			{
				FinishReadFileLocal(Task.AsyncTask->GetTask());
				ItemsToRemove.Add(TaskIdx);
				UE_LOGF(LogHTTPChunkInstaller, VeryVerbose, "Title Task Complete: %ls", *Task.Filename);
			}
			else
			{
				const int64 NewValue = Task.BytesRead.GetValue();
				if (NewValue != Task.LastBytesRead)
				{
					Task.LastBytesRead = NewValue;
					TriggerOnReadFileProgressDelegates(Task.Filename, NewValue);
				}
			}
		}

		// Clean up any tasks that were completed
		for (int32 ItemIdx = ItemsToRemove.Num() - 1; ItemIdx >= 0; ItemIdx--)
		{
			const int32 TaskIdx = ItemsToRemove[ItemIdx];
			FTitleAsyncReadData& TaskToDelete = AsyncLocalReads[TaskIdx];
			UE_LOGF(LogHTTPChunkInstaller, VeryVerbose, "Title Task Removal: %ls read: %lld", *TaskToDelete.Filename, TaskToDelete.BytesRead.GetValue());
			delete TaskToDelete.AsyncTask;
			AsyncLocalReads.RemoveAtSwap(TaskIdx);
		}
	}

	void Shutdown()
	{

	}

private:

	/** Reads the file from the local cache if it can. This is async */
	bool StartReadFileLocal(const FString& FileName)
	{
		UE_LOGF(LogHTTPChunkInstaller, VeryVerbose, "StartReadFile %ls", *FileName);
		bool bStarted = false;
		FCloudHeader* CloudFileHeader = GetCloudFileHeader(FileName);
		if (CloudFileHeader != nullptr)
		{
			// Mark file entry as in progress
			FCloudEntry* CloudFile = GetCloudFile(FileName, true);
			CloudFile->AsyncState = ECloudAsyncTaskState::InProgress;
			if (CloudFileHeader->Hash.IsEmpty())
			{
				UE_LOGF(LogHTTPChunkInstaller, Warning, "Requested file (%ls) is missing a hash, so can't be verified", *FileName);
			}
			FTitleAsyncReadData* NewItem = new FTitleAsyncReadData();
			NewItem->Filename = FileName;

			// Create the async task and start it
			NewItem->AsyncTask = new FAsyncTask<FTitleFileHttpAsyncLoadAndVerify>(FileName,
				GetLocalFilePath(FileName), CloudFileHeader->Hash, CloudFileHeader->HashType, &NewItem->BytesRead);

			AsyncLocalReads.Add(NewItem);
			NewItem->AsyncTask->StartBackgroundTask();
			bStarted = true;
		}
		return bStarted;
	}
	/** Completes the async operation of the local file read */
	void FinishReadFileLocal(FTitleFileHttpAsyncLoadAndVerify& AsyncLoad)
	{
		UE_LOGF(LogHTTPChunkInstaller, VeryVerbose, "FinishReadFileLocal %ls", *AsyncLoad.OriginalFileName);
		FCloudHeader* CloudFileHeader = GetCloudFileHeader(AsyncLoad.OriginalFileName);
		FCloudEntry* CloudFile = GetCloudFile(AsyncLoad.OriginalFileName, true);
		if (CloudFileHeader != nullptr && CloudFile != nullptr)
		{
			// if hash matches then just use the local file
			if (AsyncLoad.bHashesMatched)
			{
				UE_LOGF(LogHTTPChunkInstaller, Verbose, "Local file hash matches cloud header. No need to download for filename=%ls", *AsyncLoad.OriginalFileName);
				CloudFile->Data = AsyncLoad.FileData;
				CloudFile->AsyncState = ECloudAsyncTaskState::Done;
				TriggerOnReadFileProgressDelegates(AsyncLoad.OriginalFileName, static_cast<uint64>(AsyncLoad.BytesRead->GetValue()));
				TriggerOnReadFileCompleteDelegates(true, AsyncLoad.OriginalFileName);
			}
			else
			{
				// Request it from server
				ReadFileRemote(AsyncLoad.OriginalFileName);
			}
		}
		else
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "ReadFile request failed for file (%ls)", *AsyncLoad.OriginalFileName);
			TriggerOnReadFileCompleteDelegates(false, AsyncLoad.OriginalFileName);
		}
	}
	/** Requests the file from MCP. This is async */
	bool ReadFileRemote(const FString& FileName)
	{
		UE_LOGF(LogHTTPChunkInstaller, VeryVerbose, "ReadFileRemote %ls", *FileName);

		bool bStarted = false;
		FCloudHeader* CloudFileHeader = GetCloudFileHeader(FileName);
		if (CloudFileHeader != nullptr)
		{
			FCloudEntry* CloudFile = GetCloudFile(FileName, true);
			CloudFile->Data.Empty();
			CloudFile->AsyncState = ECloudAsyncTaskState::InProgress;

			// Create the Http request and add to pending request list
			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
			FileRequests.Add(HttpRequest, FPendingFileRequest(FileName));
			FileProgressRequestsMap.Add(HttpRequest, FPendingFileRequest(FileName));

			HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineTitleFileHttp::ReadFile_HttpRequestComplete);
			HttpRequest->OnRequestProgress64().BindRaw(this, &FOnlineTitleFileHttp::ReadFile_HttpRequestProgress);
			FString RequestUrl;
			// Grab the file from the specified URL if that was set, otherwise use the old method that hits the game service
			if (CloudFileHeader != nullptr && !CloudFileHeader->URL.IsEmpty())
			{
				RequestUrl = CloudFileHeader->URL;
			}
			else
			{
				RequestUrl = GetBaseUrl() + FileName;
			}
			HttpRequest->SetURL(RequestUrl);
			HttpRequest->SetVerb(TEXT("GET"));
			bStarted = HttpRequest->ProcessRequest();

			if (!bStarted)
			{
				UE_LOGF(LogHTTPChunkInstaller, Error, "Unable to start the HTTP request to fetch file (%ls)", *FileName);
			}
		}
		else
		{
			UE_LOGF(LogHTTPChunkInstaller, Error, "No cloud file header entry for filename=%ls.", *FileName);
		}
		return bStarted;
	}

	/**
	* Delegate called when a Http request completes for enumerating list of file headers
	*/
	void EnumerateFiles_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
	{
		const FCloudPagedQuery& PendingOp = EnumerateFilesRequests.FindRef(HttpRequest);
		EnumerateFilesRequests.Remove(HttpRequest);

		bool bResult = false;
		FString ResponseStr, ErrorStr;

		if (HttpResponse.IsValid() && EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			ResponseStr = HttpResponse->GetContentAsString();
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "EnumerateFiles request complete. url=%ls code=%d response=%ls",
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);

			if (PendingOp.Start == 0)
			{
				FileHeaders.Empty();
			}

			// parse the html for the file list
			if (ResponseStr.StartsWith(TEXT("<!DOCTYPE")))
			{
				TArray<FString> Lines;
				ResponseStr.ParseIntoArrayLines(Lines);
				for (int Index = 0; Index < Lines.Num(); ++Index)
				{
					if (Lines[Index].StartsWith(TEXT("<li>")))
					{
						TArray<FString> Elements;
						Lines[Index].ParseIntoArray(Elements, TEXT(">"));
						if (!Elements[2].StartsWith(TEXT("Chunks")))
						{
							FString File = Elements[2].Replace(TEXT("</a"), TEXT(""));
							FCloudHeader FileHeader;
							FileHeader.DLName = File;
							FileHeader.FileName = File;
							FileHeader.URL = GetBaseUrl() + EnumerateFilesUrl + TEXT("/") + File;
							FileHeader.Hash.Empty();
							FileHeader.FileSize = 0;
							FileHeaders.Add(FileHeader);
						}
					}
				}
				bResult = true;
			}
			else
			{
				// Create the Json parser
				TSharedPtr<FJsonObject> JsonObject;
				TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(ResponseStr);

				if (FJsonSerializer::Deserialize(JsonReader, JsonObject) &&
					JsonObject.IsValid())
				{
					// Parse the array of file headers
					TArray<TSharedPtr<FJsonValue> > JsonFileHeaders = JsonObject->GetArrayField(TEXT("files"));
					for (TArray<TSharedPtr<FJsonValue> >::TConstIterator It(JsonFileHeaders); It; ++It)
					{
						TSharedPtr<FJsonObject> JsonFileHeader = (*It)->AsObject();
						if (JsonFileHeader.IsValid())
						{
							FCloudHeader FileHeader;
							if (JsonFileHeader->HasField(TEXT("hash")))
							{
								FileHeader.Hash = JsonFileHeader->GetStringField(TEXT("hash"));
								FileHeader.HashType = FileHeader.Hash.IsEmpty() ? NAME_None : NAME_SHA1;
							}
							// This one takes priority over the old SHA1 hash if present (requires platform support)
							if (bPlatformSupportsSHA256 && JsonFileHeader->HasField(TEXT("hash256")))
							{
								FString Hash256 = JsonFileHeader->GetStringField(TEXT("hash256"));
								if (!Hash256.IsEmpty())
								{
									FileHeader.Hash = Hash256;
									FileHeader.HashType = FileHeader.Hash.IsEmpty() ? NAME_None : NAME_SHA256;
								}
							}
							if (JsonFileHeader->HasField(TEXT("uniqueFilename")))
							{
								FileHeader.DLName = JsonFileHeader->GetStringField(TEXT("uniqueFilename"));
							}
							if (JsonFileHeader->HasField(TEXT("filename")))
							{
								FileHeader.FileName = JsonFileHeader->GetStringField(TEXT("filename"));
							}
							if (JsonFileHeader->HasField(TEXT("length")))
							{
								FileHeader.FileSize = FMath::TruncToInt(JsonFileHeader->GetNumberField(TEXT("length")));
							}
							if (JsonFileHeader->HasField(TEXT("URL")))
							{
								FileHeader.URL = GetBaseUrl() + EnumerateFilesUrl + TEXT("/") + JsonFileHeader->GetStringField(TEXT("URL"));
							}

							if (FileHeader.FileName.IsEmpty())
							{
								FileHeader.FileName = FileHeader.DLName;
							}

							if (FileHeader.Hash.IsEmpty() ||
								(FileHeader.DLName.IsEmpty() && FileHeader.URL.IsEmpty()) ||
								FileHeader.HashType == NAME_None)
							{
								UE_LOGF(LogHTTPChunkInstaller, Warning, "Invalid file entry hash=%ls hashType=%ls dlname=%ls filename=%ls URL=%ls",
									*FileHeader.Hash,
									*FileHeader.HashType.ToString(),
									*FileHeader.DLName,
									*FileHeader.FileName,
									*FileHeader.URL);
							}
							else
							{
								int32 FoundIdx = INDEX_NONE;
								for (int32 Idx = 0; Idx < FileHeaders.Num(); Idx++)
								{
									const FCloudHeader& ExistingFile = FileHeaders[Idx];
									if (ExistingFile.DLName == FileHeader.DLName)
									{
										FoundIdx = Idx;
										break;
									}
								}
								if (FoundIdx != INDEX_NONE)
								{
									FileHeaders[FoundIdx] = FileHeader;
								}
								else
								{
									FileHeaders.Add(FileHeader);
								}
							}
						}
					}
				}
				bResult = true;
			}
		}
		else
		{
			if (HttpResponse.IsValid())
			{
				ErrorStr = FText::Format(LOCTEXT("HttpResponseFromUrl", "HTTP {0} response from {1}"),
					FText::AsNumber(HttpResponse->GetResponseCode()),
					FText::FromString(HttpResponse->GetURL())).ToString();
			}
			else
			{
				ErrorStr = FText::Format(LOCTEXT("HttpResponse_Failed", "Connection to {0} failed"), FText::FromString(HttpRequest->GetURL())).ToString();
			}
		}

		if (!ErrorStr.IsEmpty())
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "EnumerateFiles request failed. %ls", *ErrorStr);
		}
		else
		{
			// Everything went ok, so we can remove any cached files that are not in the current list
			DeleteCachedFiles(true);
		}

		TriggerOnEnumerateFilesCompleteDelegates(bResult);
	}

	/**
	* Delegate called when a Http request completes for reading a cloud file
	*/
	void ReadFile_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
	{
		bool bResult = false;
		FString ResponseStr, ErrorStr;

		// should have a pending Http request
		FPendingFileRequest PendingRequest = FileRequests.FindChecked(HttpRequest);
		FileRequests.Remove(HttpRequest);
		// remove from progress updates
		FileProgressRequestsMap.Remove(HttpRequest);
		HttpRequest->OnRequestProgress64().Unbind();

		// Cloud file being operated on
		FCloudEntry* CloudFile = GetCloudFile(PendingRequest.FileName, true);
		CloudFile->AsyncState = ECloudAsyncTaskState::Failed;
		CloudFile->Data.Empty();

		if (HttpResponse.IsValid() && EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "ReadFile request complete. url=%ls code=%d",
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode());

			// update the memory copy of the file with data that was just downloaded
			CloudFile->AsyncState = ECloudAsyncTaskState::Done;
			CloudFile->Data = HttpResponse->GetContent();

			if (bCacheFiles)
			{
				// cache to disk on successful download
				SaveCloudFileToDisk(CloudFile->FileName, CloudFile->Data);
			}

			bResult = true;
		}
		else
		{
			if (HttpResponse.IsValid())
			{
				ErrorStr = FText::Format(LOCTEXT("HttpResponseFromUrl", "HTTP {0} response from {1}"),
					FText::AsNumber(HttpResponse->GetResponseCode()),
					FText::FromString(HttpResponse->GetURL())).ToString();
			}
			else
			{
				ErrorStr = FText::Format(LOCTEXT("HttpResponse_Failed", "Connection to {0} failed"), FText::FromString(HttpRequest->GetURL())).ToString();
			}
		}

		if (!ErrorStr.IsEmpty())
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "EnumerateFiles request failed. %ls", *ErrorStr);
		}
		TriggerOnReadFileCompleteDelegates(bResult, PendingRequest.FileName);
	}

	/**
	* Delegate called as a Http request progresses for reading a cloud file
	*/
	void ReadFile_HttpRequestProgress(FHttpRequestPtr HttpRequest, uint64 BytesSent, uint64 BytesReceived)
	{
		FPendingFileRequest PendingRequest = FileProgressRequestsMap.FindChecked(HttpRequest);
		// Just forward this to anyone that is listening
		TriggerOnReadFileProgressDelegates(PendingRequest.FileName, BytesReceived);
	}

	/**
	* Find/create cloud file entry
	*
	* @param FileName cached file entry to find
	* @param bCreateIfMissing create the file entry if not found
	*
	* @return cached cloud file entry
	*/
	FCloudEntry* GetCloudFile(const FString& FileName, bool bCreateIfMissing)
	{
		FCloudEntry* CloudFile = NULL;
		for (int Idx = 0; Idx < Files.Num(); Idx++)
		{
			if (Files[Idx].FileName == FileName)
			{
				CloudFile = &Files[Idx];
				break;
			}
		}
		if (CloudFile == NULL && bCreateIfMissing)
		{
			CloudFile = new(Files)FCloudEntry(FileName);
		}
		return CloudFile;
	}

	/**
	* Find cloud file header entry
	*
	* @param FileName cached file entry to find
	*
	* @return cached cloud file header entry
	*/
	FCloudHeader* GetCloudFileHeader(const FString& FileName)
	{
		FCloudHeader* CloudFileHeader = NULL;

		for (int Idx = 0; Idx < FileHeaders.Num(); Idx++)
		{
			if (FileHeaders[Idx].DLName == FileName)
			{
				CloudFileHeader = &FileHeaders[Idx];
				break;
			}
		}

		return CloudFileHeader;
	}

	/**
	* Converts filename into a local file cache path
	*
	* @param FileName name of file being loaded
	*
	* @return unreal file path to be used by file manager
	*/
	FString GetLocalFilePath(const FString& FileName)
	{
		return GetLocalCachePath() + FileName;
	}

	/**
	* @return full path to cache directory
	*/
	FString GetLocalCachePath()
	{
		return FPaths::ProjectPersistentDownloadDir() / TEXT("EMS/");
	}

	/**
	* Save a file from a given user to disk
	*
	* @param FileName name of file being saved
	* @param Data data to write to disk
	*/
	void SaveCloudFileToDisk(const FString& Filename, const TArray<uint8>& Data)
	{
		// save local disk copy as well
		FString LocalFilePath = GetLocalFilePath(Filename);
		bool bSavedLocal = FFileHelper::SaveArrayToFile(Data, *LocalFilePath);
		if (bSavedLocal)
		{
			UE_LOGF(LogHTTPChunkInstaller, Verbose, "WriteUserFile request complete. Local file cache updated =%ls",
				*LocalFilePath);
		}
		else
		{
			UE_LOGF(LogHTTPChunkInstaller, Warning, "WriteUserFile request complete. Local file cache failed to update =%ls",
				*LocalFilePath);
		}
	}

	/**
	* Should use the initialization constructor instead
	*/
	FOnlineTitleFileHttp() = delete;

	/** Config based url for enumerating list of cloud files*/
	FString EnumerateFilesUrl;
	/** Config based url for accessing the HTTP server */
	FString BaseUrl;

	FString GetBaseUrl()
	{
		return TEXT("http://") + BaseUrl + TEXT("/");
	}

	/** List of pending Http requests for enumerating files */
	TMap<FHttpRequestPtr, FCloudPagedQuery> EnumerateFilesRequests;

	/** Info used to send request for a file */
	struct FPendingFileRequest
	{
		/**
		* Constructor
		*/
		FPendingFileRequest(const FString& InFileName = FString(TEXT("")))
			: FileName(InFileName)
		{

		}

		/**
		* Equality op
		*/
		inline bool operator==(const FPendingFileRequest& Other) const
		{
			return FileName == Other.FileName;
		}

		/** File being operated on by the pending request */
		FString FileName;
	};
	/** List of pending Http requests for reading files */
	TMap<FHttpRequestPtr, FPendingFileRequest> FileRequests;
	TMap<FHttpRequestPtr, FPendingFileRequest> FileProgressRequestsMap;

	TArray<FCloudHeader> FileHeaders;
	TArray<FCloudEntry> Files;
	bool bCacheFiles;
	bool bPlatformSupportsSHA256;

	/** Information about local file reads that are in progress */
	struct FTitleAsyncReadData
	{
		/** Name of the file being loaded */
		FString Filename;
		/** Amount of data that has been loaded on the async thread so far */
		FThreadSafeCounter64 BytesRead;
		/** Bytes read last time game thread noticed */
		int64 LastBytesRead;
		/** Async tasks doing the work */
		FAsyncTask<FTitleFileHttpAsyncLoadAndVerify>*  AsyncTask;

		FTitleAsyncReadData() :
			LastBytesRead(0),
			AsyncTask(nullptr)
		{ }

		bool operator==(const FTitleAsyncReadData& Other) const
		{
			return Filename == Other.Filename && AsyncTask == Other.AsyncTask;
		}
	};
	/** Holds the outstanding tasks for hitch free loading and hash calculation */
	TIndirectArray<FTitleAsyncReadData> AsyncLocalReads;
};

// Helper class to find all pak file manifests.
class FChunkSearchVisitor: public IPlatformFile::FDirectoryVisitor
{
public:
	TArray<FString>				PakManifests;

	FChunkSearchVisitor()
	{}

	virtual bool Visit(const TCHAR* FilenameOrDirectory,bool bIsDirectory)
	{
		if(bIsDirectory == false)
		{
			FString Filename(FilenameOrDirectory);
			if(FPaths::GetBaseFilename(Filename).MatchesWildcard("*.manifest"))
			{
				PakManifests.AddUnique(Filename);
			}
		}
		return true;
	}
};


FHTTPChunkInstall::FHTTPChunkInstall()
	: InstallingChunkID(-1)
	, InstallerState(ChunkInstallState::Setup)
	, InstallSpeed(EChunkInstallSpeed::Fast)
	, bFirstRun(true)
	, bSystemInitialised(false)
#if !UE_BUILD_SHIPPING
	, bDebugNoInstalledRequired(false)
#endif
{

}


FHTTPChunkInstall::~FHTTPChunkInstall()
{
	if (InstallService.IsValid())
	{
		InstallService->CancelInstall();
		InstallService.Reset();
	}
}

bool FHTTPChunkInstall::Tick(float DeltaSeconds)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FHTTPChunkInstall_Tick);

	if (!bSystemInitialised)
	{
		InitialiseSystem();
	}

	switch (InstallerState)
	{
	case ChunkInstallState::Setup:
		{
			check(OnlineTitleFile.IsValid());
			EnumFilesCompleteHandle = OnlineTitleFile->AddOnEnumerateFilesCompleteDelegate_Handle(FOnEnumerateFilesCompleteDelegate::CreateRaw(this,&FHTTPChunkInstall::OSSEnumerateFilesComplete));
			ReadFileCompleteHandle = OnlineTitleFile->AddOnReadFileCompleteDelegate_Handle(FOnReadFileCompleteDelegate::CreateRaw(this,&FHTTPChunkInstall::OSSReadFileComplete));
			ChunkSetupTask.SetupWork(BPSModule, InstallDir, ContentDir, HoldingDir, MountedPaks);
			ChunkSetupTaskThread.Reset(FRunnableThread::Create(&ChunkSetupTask, TEXT("Chunk discovery thread")));
			InstallerState = ChunkInstallState::SetupWait;
		} break;
	case ChunkInstallState::SetupWait:
		{
			if (ChunkSetupTask.IsDone())
			{
				ChunkSetupTaskThread->WaitForCompletion();
				ChunkSetupTaskThread.Reset();
				for (auto It = ChunkSetupTask.InstalledChunks.CreateConstIterator(); It; ++It)
				{
					UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to installed manifests", It.Key());
					InstalledManifests.Add(It.Key(), It.Value());
				}
				for (auto It = ChunkSetupTask.HoldingChunks.CreateConstIterator(); It; ++It)
				{
					UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to holding manifests", It.Key());
					PrevInstallManifests.Add(It.Key(), It.Value());
				}
				MountedPaks.Append(ChunkSetupTask.MountedPaks);
				InstallerState = ChunkInstallState::QueryRemoteManifests;
			}
		} break;
	case ChunkInstallState::QueryRemoteManifests:
		{			
			//Now query the title file service for the chunk manifests. This should return the list of expected chunk manifests
			check(OnlineTitleFile.IsValid());
			OnlineTitleFile->ClearFiles();
			InstallerState = ChunkInstallState::RequestingTitleFiles;
			UE_LOGF(LogHTTPChunkInstaller, Log, "Enumerating manifest files");
			OnlineTitleFile->EnumerateFiles();
		} break;
	case ChunkInstallState::SearchTitleFiles:
		{
			FString CleanName;
			TArray<FCloudHeader> FileList;
			TitleFilesToRead.Reset();
			RemoteManifests.Reset();
			ExpectedChunks.Empty();
			OnlineTitleFile->GetFileList(FileList);
			for (int32 FileIndex = 0, FileCount = FileList.Num(); FileIndex < FileCount; ++FileIndex)
			{
				if (FileList[FileIndex].FileName.MatchesWildcard(TEXT("*.manifest")))
				{
					UE_LOGF(LogHTTPChunkInstaller, Log, "Found manifest %ls", *FileList[FileIndex].FileName);
					TitleFilesToRead.Add(FileList[FileIndex]);
				}
			}
			InstallerState = ChunkInstallState::ReadTitleFiles;
		} break;
	case ChunkInstallState::ReadTitleFiles:
		{
			if (TitleFilesToRead.Num() > 0 && InstallSpeed != EChunkInstallSpeed::Paused)
			{
				if (!IsDataInFileCache(TitleFilesToRead[0].Hash))
				{
					UE_LOGF(LogHTTPChunkInstaller, Log, "Reading manifest %ls from remote source", *TitleFilesToRead[0].FileName);
					InstallerState = ChunkInstallState::WaitingOnRead;
					OnlineTitleFile->ReadFile(TitleFilesToRead[0].DLName);
				}
				else
				{
					InstallerState = ChunkInstallState::ReadComplete;
				}
			}
			else
			{
				InstallerState = ChunkInstallState::PostSetup;
			}
		} break;
	case ChunkInstallState::ReadComplete:
		{
			FileContentBuffer.Reset();
			bool bReadOK = false;
			bool bAlreadyLoaded = ManifestsInMemory.Contains(TitleFilesToRead[0].Hash);
			if (!IsDataInFileCache(TitleFilesToRead[0].Hash))
			{
				bReadOK = OnlineTitleFile->GetFileContents(TitleFilesToRead[0].DLName, FileContentBuffer);
				if (bReadOK)
				{
					AddDataToFileCache(TitleFilesToRead[0].Hash, FileContentBuffer);
				}
			}
			else if (!bAlreadyLoaded)
			{
				bReadOK = GetDataFromFileCache(TitleFilesToRead[0].Hash, FileContentBuffer);
				if (!bReadOK)
				{
					RemoveDataFromFileCache(TitleFilesToRead[0].Hash);
				}
			}
			if (bReadOK)
			{
				if (!bAlreadyLoaded)
				{
					ParseTitleFileManifest(TitleFilesToRead[0].Hash);
				}
				// Even if the Parse failed remove the file from the list
				TitleFilesToRead.RemoveAt(0);
			}
			if (TitleFilesToRead.Num() == 0)
			{
				if (bFirstRun)
				{
					ChunkMountTask.SetupWork(BPSModule, ContentDir, MountedPaks, ExpectedChunks);
					ChunkMountTaskThread.Reset(FRunnableThread::Create(&ChunkMountTask, TEXT("Chunk mounting thread")));
				}
				InstallerState = ChunkInstallState::PostSetup;
			}
			else
			{
				InstallerState = ChunkInstallState::ReadTitleFiles;
			}
		} break;
	case ChunkInstallState::EnterOfflineMode:
		{
			for (auto It = InstalledManifests.CreateConstIterator(); It; ++It)
			{
				ExpectedChunks.Add(It.Key());
			}
			ChunkMountTask.SetupWork(BPSModule, ContentDir, MountedPaks, ExpectedChunks);
			ChunkMountTaskThread.Reset(FRunnableThread::Create(&ChunkMountTask, TEXT("Chunk mounting thread")));
			InstallerState = ChunkInstallState::PostSetup;
		} break;
	case ChunkInstallState::PostSetup:
		{
			if (bFirstRun)
			{
				if (ChunkMountTask.IsDone())
				{
					ChunkMountTaskThread->WaitForCompletion();
					ChunkMountTaskThread.Reset();
					MountedPaks.Append(ChunkMountTask.MountedPaks);
					UE_LOGF(LogHTTPChunkInstaller, Log, "Completed First Run");
					bFirstRun = false;
					if (PriorityQueue.Num() == 0)
					{
						SetInstallSpeed(EChunkInstallSpeed::Paused);
					}
				}
			}
			else
			{
				InstallerState = ChunkInstallState::Idle;
			}
		} break;
	case ChunkInstallState::Idle: 
		{
			UpdatePendingInstallQueue();
		} break;
	case ChunkInstallState::CopyToContent:
		{
			if (!ChunkCopyInstall.IsDone() || !InstallService->IsComplete())
			{
				break;
			}
			check(InstallingChunkID != -1);
			if (InstallService.IsValid())
			{
				InstallService.Reset();
			}
			ChunkCopyInstallThread.Reset();
			check(RemoteManifests.Find(InstallingChunkID));
			UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to installed manifests", InstallingChunkID);
			InstalledManifests.Add(InstallingChunkID, InstallingChunkManifest);
			UE_LOGF(LogHTTPChunkInstaller, Log, "Removing Chunk %d from remote manifests", InstallingChunkID);
			RemoteManifests.Remove(InstallingChunkID, InstallingChunkManifest);
			MountedPaks.Append(ChunkCopyInstall.MountedPaks);
			if (!RemoteManifests.Contains(InstallingChunkID))
			{
				// No more manifests relating to the chunk ID are left to install.
				// Inform any listeners that the install has been completed.
				InstallDelegate.Broadcast(InstallingChunkID, true);
			}
			EndInstall();

		} break;
	case ChunkInstallState::Installing:
	case ChunkInstallState::RequestingTitleFiles:
	case ChunkInstallState::WaitingOnRead:
	default:
		break;
	}

	if (OnlineTitleFileHttp.IsValid())
	{
		static_cast<FOnlineTitleFileHttp*>(OnlineTitleFileHttp.Get())->Tick(DeltaSeconds);
	}

	return true;
}

void FHTTPChunkInstall::UpdatePendingInstallQueue()
{
	if (InstallingChunkID != -1
#if !UE_BUILD_SHIPPING
		|| bDebugNoInstalledRequired
#endif
	)
	{
		return;
	}

	check(!InstallService.IsValid());
	bool bPatch = false;
	while (PriorityQueue.Num() > 0 && InstallerState != ChunkInstallState::Installing)
	{
		const FChunkPrio& NextChunk = PriorityQueue[0];
		TArray<IBuildManifestPtr> FoundChunkManifests;
		RemoteManifests.MultiFind(NextChunk.ChunkID, FoundChunkManifests);
		if (FoundChunkManifests.Num() > 0)
		{
			auto ChunkManifest = FoundChunkManifests[0];
			auto ChunkIDField = ChunkManifest->GetCustomField(TEXT("ChunkID"));
			if (ChunkIDField.IsValid())
			{
				BeginChunkInstall(NextChunk.ChunkID, ChunkManifest, FindPreviousInstallManifest(ChunkManifest));
			}
			else
			{
				PriorityQueue.RemoveAt(0);
			}
		}
		else
		{
			PriorityQueue.RemoveAt(0);
		}
	}
	if (InstallingChunkID == -1)
	{
		// Install the first available chunk
		for (auto It = RemoteManifests.CreateConstIterator(); It; ++It)
		{
			if (It)
			{
				IBuildManifestPtr ChunkManifest = It.Value();
				auto ChunkIDField = ChunkManifest->GetCustomField(TEXT("ChunkID"));
				if (ChunkIDField.IsValid())
				{
					BeginChunkInstall(ChunkIDField->AsInteger(), ChunkManifest, FindPreviousInstallManifest(ChunkManifest));
					return;
				}
			}
		}
	}
}


EChunkLocation::Type FHTTPChunkInstall::GetChunkLocation(uint32 ChunkID)
{
#if !UE_BUILD_SHIPPING
	if(bDebugNoInstalledRequired)
	{
		return EChunkLocation::BestLocation;
	}
#endif

	// Safe to assume Chunk0 is ready
	if (ChunkID == 0)
	{
		return EChunkLocation::BestLocation;
	}

	if (bFirstRun || !bSystemInitialised)
	{
		/** Still waiting on setup to finish, report that nothing is installed yet... */
		return EChunkLocation::NotAvailable;
	}
	TArray<IBuildManifestPtr> FoundManifests;
	RemoteManifests.MultiFind(ChunkID, FoundManifests);
	if (FoundManifests.Num() > 0)
	{
		return EChunkLocation::NotAvailable;
	}

	InstalledManifests.MultiFind(ChunkID, FoundManifests);
	if (FoundManifests.Num() > 0)
	{
		return EChunkLocation::BestLocation;
	}

	return EChunkLocation::DoesNotExist;
}


float FHTTPChunkInstall::GetChunkProgress(uint32 ChunkID,EChunkProgressReportingType::Type ReportType)
{
#if !UE_BUILD_SHIPPING
	if (bDebugNoInstalledRequired)
	{
		return 100.f;
	}
#endif

	// Safe to assume Chunk0 is ready
	if (ChunkID == 0)
	{
		return 100.f;
	}

	if (bFirstRun || !bSystemInitialised)
	{
		/** Still waiting on setup to finish, report that nothing is installed yet... */
		return 0.f;
	}
	TArray<IBuildManifestPtr> FoundManifests;
	RemoteManifests.MultiFind(ChunkID, FoundManifests);
	if (FoundManifests.Num() > 0)
	{
		float Progress = 0;
		if (InstallingChunkID == ChunkID && InstallService.IsValid())
		{
			Progress = InstallService->GetUpdateProgress();
		}
		return Progress / FoundManifests.Num();
	}

	InstalledManifests.MultiFind(ChunkID, FoundManifests);
	if (FoundManifests.Num() > 0)
	{
		return 100.f;
	}

	return 0.f;
}

void FHTTPChunkInstall::OSSEnumerateFilesComplete(bool bSuccess)
{
	InstallerState = bSuccess ? ChunkInstallState::SearchTitleFiles : ChunkInstallState::EnterOfflineMode;
}

void FHTTPChunkInstall::OSSReadFileComplete(bool bSuccess, const FString& Filename)
{
	InstallerState = bSuccess ? ChunkInstallState::ReadComplete : ChunkInstallState::EnterOfflineMode;
}

void FHTTPChunkInstall::OSSInstallComplete(const IBuildInstallerRef& Installer)
{
	if (Installer->CompletedSuccessfully())
	{
		if (!ensure(Installer->GetConfiguration().InstallerActions.Num() > 0))
		{
			EndInstall();
			return;
		}

		IBuildManifestRef BuildManifest = Installer->GetConfiguration().InstallerActions[0].GetInstallManifest();

		// Completed OK. Write the manifest. If the chunk doesn't exist, copy to the content dir.
		// Otherwise, writing the manifest will prompt a copy on next start of the game
		FString ManifestName;
		FString ChunkFdrName;
		uint32 ChunkID;
		bool bIsPatch;
		if (!BuildChunkFolderName(BuildManifest, ChunkFdrName, ManifestName, ChunkID, bIsPatch))
		{
			//Something bad has happened, bail
			EndInstall();
			return;
		}
		UE_LOGF(LogHTTPChunkInstaller, Log, "Chunk %d install complete, preparing to copy to content directory", ChunkID);
		FString ManifestPath = FPaths::Combine(*InstallDir, *ChunkFdrName, *ManifestName);
		FString HoldingManifestPath = FPaths::Combine(*HoldingDir, *ChunkFdrName, *ManifestName);
		FString SrcDir = FPaths::Combine(*InstallDir, *ChunkFdrName);
		FString DestDir = FPaths::Combine(*ContentDir, *ChunkFdrName);
		bool bCopyDir = InstallDir != ContentDir;
		TArray<IBuildManifestPtr> FoundManifests;
		InstalledManifests.MultiFind(ChunkID, FoundManifests);
		for (const auto& It : FoundManifests)
		{
			auto FoundPatchField = It->GetCustomField(TEXT("bIsPatch"));
			bool bFoundPatch = FoundPatchField.IsValid() ? FoundPatchField->AsString() == TEXT("true") : false;
			if (bFoundPatch == bIsPatch)
			{
				bCopyDir = false;
			}
		}
		ChunkCopyInstall.SetupWork(ManifestPath, HoldingManifestPath, SrcDir, DestDir, BPSModule, BuildManifest, MountedPaks, bCopyDir);
		UE_LOGF(LogHTTPChunkInstaller, Log, "Copying Chunk %d to content directory", ChunkID);
		ChunkCopyInstallThread.Reset(FRunnableThread::Create(&ChunkCopyInstall, TEXT("Chunk Install Copy Thread")));
		InstallerState = ChunkInstallState::CopyToContent;
	}
	else
	{
		//Something bad has happened, return to the Idle state. We'll re-attempt the install
		EndInstall();
	}
}

void FHTTPChunkInstall::ParseTitleFileManifest(const FString& ManifestFileHash)
{
#if !UE_BUILD_SHIPPING
	if (bDebugNoInstalledRequired)
	{
		// Forces the installer to think that no remote manifests exist, so nothing needs to be installed.
		return;
	}
#endif
	auto RemoteManifest = BPSModule->MakeManifestFromData(FileContentBuffer);
	if (!RemoteManifest.IsValid())
	{
		UE_LOGF(LogHTTPChunkInstaller, Warning, "Manifest was invalid");
		return;
	}
	auto RemoteChunkIDField = RemoteManifest->GetCustomField(TEXT("ChunkID"));
	if (!RemoteChunkIDField.IsValid())
	{
		UE_LOGF(LogHTTPChunkInstaller, Warning, "Manifest ChunkID was invalid or missing");
		return;
	}
	//Compare to installed manifests and add to the remote if it needs to be installed.
	uint32 ChunkID = (uint32)RemoteChunkIDField->AsInteger();
	ExpectedChunks.Add(ChunkID);
	TArray<IBuildManifestPtr> FoundManifests;
	InstalledManifests.MultiFind(ChunkID, FoundManifests);
	uint32 FoundCount = FoundManifests.Num();
	if (FoundCount > 0)
	{
		auto RemotePatchManifest = RemoteManifest->GetCustomField(TEXT("bIsPatch"));
		auto RemoteVersion = RemoteManifest->GetVersionString();
		bool bRemoteIsPatch = RemotePatchManifest.IsValid() ? RemotePatchManifest->AsString() == TEXT("true") : false;
		for (uint32 FoundIndex = 0; FoundIndex < FoundCount; ++FoundIndex)
		{
			const auto& InstalledManifest = FoundManifests[FoundIndex];
			auto InstalledVersion = InstalledManifest->GetVersionString();
			auto InstallPatchManifest = InstalledManifest->GetCustomField(TEXT("bIsPatch"));
			bool bInstallIsPatch = InstallPatchManifest.IsValid() ? InstallPatchManifest->AsString() == TEXT("true") : false;
			if (InstalledVersion != RemoteVersion && bInstallIsPatch == bRemoteIsPatch)
			{
				UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to remote manifests", ChunkID);
				RemoteManifests.Add(ChunkID, RemoteManifest);
				if(!ManifestFileHash.IsEmpty())
				{
					ManifestsInMemory.Add(ManifestFileHash);
				}
				//Remove from the installed map
				if (bFirstRun)
				{
					// Prevent the paks from being mounted by removing the manifest file
					FString ChunkFdrName;
					FString ManifestName;
					bool bIsPatch;
					if (BuildChunkFolderName(InstalledManifest.ToSharedRef(), ChunkFdrName, ManifestName, ChunkID, bIsPatch))
					{
						FString ManifestPath = FPaths::Combine(*ContentDir, *ChunkFdrName, *ManifestName);
						FString HoldingPath = FPaths::Combine(*HoldingDir, *ChunkFdrName, *ManifestName);
						IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
						PlatformFile.CreateDirectoryTree(*FPaths::Combine(*HoldingDir, *ChunkFdrName));
						PlatformFile.MoveFile(*HoldingPath, *ManifestPath);
					}
					UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to previous installed manifests", ChunkID);
					PrevInstallManifests.Add(ChunkID, InstalledManifest);
					UE_LOGF(LogHTTPChunkInstaller, Log, "Removing Chunk %d from installed manifests", ChunkID);
					InstalledManifests.Remove(ChunkID, InstalledManifest);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogHTTPChunkInstaller, Log, "Adding Chunk %d to remote manifests", ChunkID);
		RemoteManifests.Add(ChunkID, RemoteManifest);
		if (!ManifestFileHash.IsEmpty())
		{
			ManifestsInMemory.Add(ManifestFileHash);
		}
	}
}

bool FHTTPChunkInstall::BuildChunkFolderName(IBuildManifestRef Manifest, FString& ChunkFdrName, FString& ManifestName, uint32& ChunkID, bool& bIsPatch)
{
	auto ChunkIDField = Manifest->GetCustomField(TEXT("ChunkID"));
	auto ChunkPatchField = Manifest->GetCustomField(TEXT("bIsPatch"));

	if (!ChunkIDField.IsValid())
	{
		return false;
	}
	ChunkID = ChunkIDField->AsInteger();
	bIsPatch = ChunkPatchField.IsValid() ? ChunkPatchField->AsString() == TEXT("true") : false;
	ManifestName = FString::Printf(TEXT("chunk_%u"), ChunkID);
	if (bIsPatch)
	{
		ManifestName += TEXT("_patch");
	}
	ManifestName += TEXT(".manifest");
	ChunkFdrName = FString::Printf(TEXT("%s%d"), !bIsPatch ? TEXT("base") : TEXT("patch"), ChunkID);
	return true;
}

bool FHTTPChunkInstall::PrioritizeChunk(uint32 ChunkID, EChunkPriority::Type Priority)
{	
	int32 FoundIndex;
	PriorityQueue.Find(FChunkPrio(ChunkID, Priority), FoundIndex);
	if (FoundIndex != INDEX_NONE)
	{
		PriorityQueue.RemoveAt(FoundIndex);
	}
	// Low priority is assumed if the chunk ID doesn't exist in the queue
	if (Priority != EChunkPriority::Low)
	{
		PriorityQueue.AddUnique(FChunkPrio(ChunkID, Priority));
		PriorityQueue.Sort();
	}
	return true;
}

void FHTTPChunkInstall::BeginChunkInstall(uint32 ChunkID,IBuildManifestPtr ChunkManifest, IBuildManifestPtr PrevInstallChunkManifest)
{
	check(ChunkManifest->GetCustomField(TEXT("ChunkID")).IsValid());
	InstallingChunkID = ChunkID;
	check(ChunkID > 0);
	InstallingChunkManifest = ChunkManifest;
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	auto PatchField = ChunkManifest->GetCustomField(TEXT("bIsPatch"));
	bool bIsPatch = PatchField.IsValid() ? PatchField->AsString() == TEXT("true") : false;
	auto ChunkFolderName = FString::Printf(TEXT("%s%d"),!bIsPatch ? TEXT("base") : TEXT("patch"), InstallingChunkID);
	auto ChunkInstallDir = FPaths::Combine(*InstallDir,*ChunkFolderName);
	auto ChunkStageDir = FPaths::Combine(*StageDir,*ChunkFolderName);
	if(!PlatformFile.DirectoryExists(*ChunkStageDir))
	{
		PlatformFile.CreateDirectoryTree(*ChunkStageDir);
	}
	if(!PlatformFile.DirectoryExists(*ChunkInstallDir))
	{
		PlatformFile.CreateDirectoryTree(*ChunkInstallDir);
	}

	TArray<BuildPatchServices::FInstallerAction> InstallerActions;
	InstallerActions.Add(BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(PrevInstallChunkManifest, ChunkManifest.ToSharedRef(), TSet<FString>()));

	BuildPatchServices::FBuildInstallerConfiguration Configuration(MoveTemp(InstallerActions));
	Configuration.StagingDirectory = ChunkStageDir;
	Configuration.CloudDirectories = { CloudDir + TEXT("/") + CloudDirectory };
	Configuration.InstallDirectory = ChunkInstallDir;

	UE_LOGF(LogHTTPChunkInstaller,Log,"Starting Chunk %d install",InstallingChunkID);
	InstallService = BPSModule->CreateBuildInstaller(Configuration, FBuildPatchInstallerDelegate::CreateRaw(this, &FHTTPChunkInstall::OSSInstallComplete));
	InstallService->StartInstallation();
	if(InstallSpeed == EChunkInstallSpeed::Paused && !InstallService->IsPaused())
	{
		InstallService->TogglePauseInstall();
	}
	InstallerState = ChunkInstallState::Installing;
}

/**
 * Note: the following cache functions are synchronous and may need to become asynchronous...
 */
bool FHTTPChunkInstall::AddDataToFileCache(const FString& ManifestHash,const TArray<uint8>& Data)
{
	if (ManifestHash.IsEmpty())
	{
		return false;
	}
	UE_LOGF(LogHTTPChunkInstaller, Log, "Adding data hash %ls to file cache", *ManifestHash);
	return FFileHelper::SaveArrayToFile(Data, *FPaths::Combine(*CacheDir, *ManifestHash));
}

bool FHTTPChunkInstall::IsDataInFileCache(const FString& ManifestHash)
{
	if(ManifestHash.IsEmpty())
	{
		return false;
	}
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	return PlatformFile.FileExists(*FPaths::Combine(*CacheDir, *ManifestHash));
}

bool FHTTPChunkInstall::GetDataFromFileCache(const FString& ManifestHash, TArray<uint8>& Data)
{
	if(ManifestHash.IsEmpty())
	{
		return false;
	}
	UE_LOGF(LogHTTPChunkInstaller, Log, "Reading data hash %ls from file cache", *ManifestHash);
	return FFileHelper::LoadFileToArray(Data, *FPaths::Combine(*CacheDir, *ManifestHash));
}

bool FHTTPChunkInstall::RemoveDataFromFileCache(const FString& ManifestHash)
{
	if(ManifestHash.IsEmpty())
	{
		return false;
	}
	UE_LOGF(LogHTTPChunkInstaller, Log, "Removing data hash %ls from file cache", *ManifestHash);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	auto ManifestPath = FPaths::Combine(*CacheDir, *ManifestHash);
	if (PlatformFile.FileExists(*ManifestPath))
	{
		return PlatformFile.DeleteFile(*ManifestPath);
	}
	return false;
}

void FHTTPChunkInstall::InitialiseSystem()
{
	BPSModule = GetBuildPatchServices();

#if !UE_BUILD_SHIPPING
	const TCHAR* CmdLine = FCommandLine::Get();
	if (!FPlatformProperties::RequiresCookedData() || FParse::Param(CmdLine, TEXT("NoPak")) || FParse::Param(CmdLine, TEXT("NoChunkInstall")))
	{
		bDebugNoInstalledRequired = true;
	}
#endif

	// Grab the title file interface
	FString TitleFileSource;
	bool bValidTitleFileSource = GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("TitleFileSource"), TitleFileSource, GEngineIni);
	if (bValidTitleFileSource && TitleFileSource == TEXT("Http"))
	{
		OnlineTitleFile = OnlineTitleFileHttp = MakeShareable(new FOnlineTitleFileHttp(CloudDir));
	}
/*	else if (bValidTitleFileSource && TitleFileSource != TEXT("Local"))
	{
		OnlineTitleFile = Online::GetTitleFileInterface(*TitleFileSource);
	}*/
	else
	{
		FString LocalTileFileDirectory = FPaths::ProjectConfigDir();
		auto bGetConfigDir = GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("LocalTitleFileDirectory"), LocalTileFileDirectory, GEngineIni);
		OnlineTitleFile = MakeShareable(new FLocalTitleFile(LocalTileFileDirectory));
#if !UE_BUILD_SHIPPING
		bDebugNoInstalledRequired = !bGetConfigDir;
#endif
	}
	CloudDirectory = TEXT("");
	CloudDir = FPaths::Combine(*FPaths::ProjectContentDir(), TEXT("Cloud"));
	StageDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Staged"));
	InstallDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Installed")); // By default this should match ContentDir
	BackupDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Backup"));
	CacheDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Cache"));
	HoldingDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Hold"));
	ContentDir = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Chunks"), TEXT("Installed")); // By default this should match InstallDir

	FString TmpString1;
	FString TmpString2;
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("CloudDirectory"), TmpString1, GEngineIni))
	{
		CloudDirectory = CloudDir = TmpString1;
	}
	if ((GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("CloudProtocol"), TmpString1, GEngineIni)) && (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("CloudDomain"), TmpString2, GEngineIni)))
	{
		CloudDir = FString::Printf(TEXT("%s://%s"), *TmpString1, *TmpString2);
	}
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("StageDirectory"), TmpString1, GEngineIni))
	{
		StageDir = TmpString1;
	}
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("InstallDirectory"), TmpString1, GEngineIni))
	{
		InstallDir = TmpString1;
	}
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("BackupDirectory"), TmpString1, GEngineIni))
	{
		BackupDir = TmpString1;
	}
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("ContentDirectory"), TmpString1, GEngineIni))
	{
		ContentDir = TmpString1;
	}
	if (GConfig->GetString(TEXT("HTTPChunkInstall"), TEXT("HoldingDirectory"), TmpString1, GEngineIni))
	{
		HoldingDir = TmpString1;
	}

	bFirstRun = true;
	bSystemInitialised = true;
}

IBuildManifestPtr FHTTPChunkInstall::FindPreviousInstallManifest(const IBuildManifestPtr& ChunkManifest)
{
	auto ChunkIDField = ChunkManifest->GetCustomField(TEXT("ChunkID"));
	if (!ChunkIDField.IsValid())
	{
		return IBuildManifestPtr();
	}
	auto ChunkID = ChunkIDField->AsInteger();
	TArray<IBuildManifestPtr> FoundManifests;
	PrevInstallManifests.MultiFind(ChunkID, FoundManifests);
	return FoundManifests.Num() == 0 ? IBuildManifestPtr() : FoundManifests[0];
}

void FHTTPChunkInstall::EndInstall()
{
	if (InstallService.IsValid())
	{
		//InstallService->CancelInstall();
		InstallService.Reset();
	}
	InstallingChunkID = -1;
	InstallingChunkManifest.Reset();
	InstallerState = ChunkInstallState::Idle;
}

/**
 * Module for the HTTP Chunk Installer
 */
class FHTTPChunkInstallerModule : public IPlatformChunkInstallModule
{
public:
	TUniquePtr<IPlatformChunkInstall> ChunkInstaller;

	FHTTPChunkInstallerModule()
		: ChunkInstaller(new FHTTPChunkInstall())
	{

	}

	virtual IPlatformChunkInstall* GetPlatformChunkInstall()
	{
		return ChunkInstaller.Get();
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FHTTPChunkInstallerModule, HTTPChunkInstaller);
