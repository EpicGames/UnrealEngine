// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalTitleFile.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

FLocalTitleFile::FLocalTitleFile(const FString& InRootDirectory)
: RootDirectory(InRootDirectory)
{
	RootDirectory.ReplaceInline(TEXT("\\"), TEXT("/"));
	if (!RootDirectory.EndsWith(TEXT("/")))
	{
		RootDirectory.Append(TEXT("/"));
	}
}


bool FLocalTitleFile::GetFileContents(const FString& DLName, TArray<uint8>& FileContents)
{
	const TArray< uint8 >* ExistingFileContents = DLNameToFileContents.Find(DLName);

	if (ExistingFileContents == nullptr)
	{
		return false;
	}

	FileContents.Append(*ExistingFileContents);

	return true;
}


bool FLocalTitleFile::ClearFiles()
{
	FileHeaders.Empty();
	DLNameToFileContents.Empty();

	return true;
}


bool FLocalTitleFile::ClearFile(const FString& DLName)
{
	bool ClearedFile = false;
	const FString FileName = GetFileNameFromDLName(DLName);

	for (int Index = 0; Index < FileHeaders.Num(); Index++)
	{
		if (FileHeaders[Index].DLName == DLName)
		{
			FileHeaders.RemoveAt(Index);
			ClearedFile = true;
		}
	}

	DLNameToFileContents.Remove(DLName);

	return ClearedFile;
}

void FLocalTitleFile::DeleteCachedFiles(bool bSkipEnumerated)
{
	// not implemented
}

bool FLocalTitleFile::EnumerateFiles(const FCloudPagedQuery& Page)
{
	const FString WildCard = FPaths::Combine(*RootDirectory, TEXT("*"));

	TArray<FString> Filenames;
	IFileManager::Get().FindFiles(Filenames, *WildCard, true, false);

	for(int32 FileIdx = 0; FileIdx < Filenames.Num(); ++FileIdx)
	{
		const FString Filename = Filenames[FileIdx];

		FCloudHeader NewHeader;
		NewHeader.FileName = Filename;
		NewHeader.DLName = Filename + FString::Printf(TEXT("%d"),FileIdx);
		NewHeader.FileSize = 0;
		NewHeader.Hash.Empty();

		FileHeaders.Add(NewHeader);
	}

	TriggerOnEnumerateFilesCompleteDelegates(true);

	return true;
}

void FLocalTitleFile::GetFileList(TArray<FCloudHeader>& InFileHeaders)
{
	InFileHeaders.Append(FileHeaders);
}


bool FLocalTitleFile::ReadFile(const FString& DLName)
{
	const TArray< uint8 >* ExistingFileContents = DLNameToFileContents.Find(DLName);

	if (ExistingFileContents != nullptr)
	{
		TriggerOnReadFileCompleteDelegates(true, DLName);

		return true;
	}

	const FString FileName = GetFileNameFromDLName(DLName);

	TArray<uint8> FileContents;

	if (!FFileHelper::LoadFileToArray(FileContents, *(RootDirectory + FileName)))
	{
		TriggerOnReadFileCompleteDelegates(false, DLName);

		return false;
	}

	DLNameToFileContents.Add(DLName, FileContents);
	TriggerOnReadFileCompleteDelegates(true, DLName);

	return true;
}


FString FLocalTitleFile::GetFileNameFromDLName(const FString& DLName) const
{
	for (int Index = 0; Index < FileHeaders.Num(); Index++)
	{
		if (FileHeaders[Index].DLName == DLName)
		{
			return FileHeaders[Index].FileName;
		}
	}

	return FString();
}
