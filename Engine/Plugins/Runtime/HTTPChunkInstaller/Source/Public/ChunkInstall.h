// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CoreDelegates.h"
#include "HAL/FileManager.h"
#include "Stats/Stats.h"

class FChunkInstallTask : public FRunnable
{
public:
	/** Input parameters */
	FString							ManifestPath;
	FString							HoldingManifestPath;
	FString							SrcDir;
	FString							DestDir;
	IBuildPatchServicesModule*		BPSModule;
	IBuildManifestPtr				BuildManifest;
	bool							bCopy;
	const TArray<FString>*			CurrentMountPaks;
	FEvent*							CompleteEvent;				
	/** Output */
	TArray<FString>					MountedPaks;

	FChunkInstallTask()
	{
		CompleteEvent = FPlatformProcess::GetSynchEventFromPool(true);
	}

	~FChunkInstallTask()
	{
		FPlatformProcess::ReturnSynchEventToPool(CompleteEvent);
		CompleteEvent = nullptr;
	}

	void SetupWork(FString InManifestPath, FString InHoldingManifestPath, FString InSrcDir, FString InDestDir, IBuildPatchServicesModule* InBPSModule, IBuildManifestRef InBuildManifest, const TArray<FString>& InCurrentMountedPaks, bool bInCopy)
	{
		ManifestPath = InManifestPath;
		HoldingManifestPath = InHoldingManifestPath;
		SrcDir = InSrcDir;
		DestDir = InDestDir;
		BPSModule = InBPSModule;
		BuildManifest = InBuildManifest;
		CurrentMountPaks = &InCurrentMountedPaks;
		bCopy = bInCopy;

		MountedPaks.Reset();
		CompleteEvent->Reset();
	}

	void DoWork()
	{
		// Helper class to find all pak files.
		class FPakSearchVisitor : public IPlatformFile::FDirectoryVisitor
		{
			TArray<FString>& FoundPakFiles;
		public:
			FPakSearchVisitor(TArray<FString>& InFoundPakFiles)
				: FoundPakFiles(InFoundPakFiles)
			{}
			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
			{
				if (bIsDirectory == false)
				{
					FString Filename(FilenameOrDirectory);
					if (FPaths::GetExtension(Filename) == TEXT("pak"))
					{
						FoundPakFiles.Add(Filename);
					}
				}
				return true;
			}
		};

		check(CurrentMountPaks);

		TArray<FString> PakFiles;
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		BPSModule->SaveManifestToFile(ManifestPath, BuildManifest.ToSharedRef());
		if (PlatformFile.FileExists(*HoldingManifestPath))
		{
			PlatformFile.DeleteFile(*HoldingManifestPath);
		}
		if (bCopy)
		{
			if (PlatformFile.DirectoryExists(*DestDir))
			{
				PlatformFile.DeleteDirectoryRecursively(*DestDir);
			}
			PlatformFile.CreateDirectoryTree(*DestDir);
			if (PlatformFile.CopyDirectoryTree(*DestDir, *SrcDir, true))
			{
				PlatformFile.DeleteDirectoryRecursively(*SrcDir);
			}
		}
		// Find all pak files.
		FPakSearchVisitor Visitor(PakFiles);
		PlatformFile.IterateDirectoryRecursively(*DestDir, Visitor);
		auto PakReadOrderField = BuildManifest->GetCustomField(TEXT("PakReadOrdering"));
		uint32 PakReadOrder = PakReadOrderField.IsValid() ? (uint32)PakReadOrderField->AsInteger() : 0;
		for (uint32 PakIndex = 0, PakCount = PakFiles.Num(); PakIndex < PakCount; ++PakIndex)
		{
			if (!CurrentMountPaks->Contains(PakFiles[PakIndex]) && !MountedPaks.Contains(PakFiles[PakIndex]))
			{
				// TODO: are we a patch?
// 				if (PakFiles[PakIndex].EndsWith("_P.pak"))
// 				{
// 					// bump the read prioritiy 
// 					++PakReadOrder;
// 				}
				if (FCoreDelegates::MountPak.IsBound())
				{
					auto bSuccess = FCoreDelegates::MountPak.Execute(PakFiles[PakIndex], PakReadOrder);
#if !UE_BUILD_SHIPPING
					if (!bSuccess)
					{
						// This can fail because of the sandbox system - which the pak system doesn't understand.
						auto SandboxedPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*PakFiles[PakIndex]);
						bSuccess = FCoreDelegates::MountPak.Execute(SandboxedPath, PakReadOrder);
					}
#endif
					MountedPaks.Add(PakFiles[PakIndex]);
				}
			}
		}
		//Register the install
		BPSModule->RegisterAppInstallation(BuildManifest.ToSharedRef(), DestDir);

		CompleteEvent->Trigger();
	}

	uint32 Run()
	{
		DoWork();
		return 0;
	}

	bool IsDone()
	{
		return CompleteEvent->Wait(FTimespan::Zero());
	}

	static const TCHAR *Name()
	{
		return TEXT("FChunkDescovery");
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChunkInstallTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};
