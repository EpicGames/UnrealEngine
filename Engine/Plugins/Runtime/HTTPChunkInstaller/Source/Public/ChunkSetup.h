// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/Runnable.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"

// Helper class to find all pak/mainfests files.
class FFileSearchVisitor : public IPlatformFile::FDirectoryVisitor
{
	FString			 FileWildcard;
	TArray<FString>& FoundFiles;
public:
	FFileSearchVisitor(FString InFileWildcard, TArray<FString>& InFoundFiles)
		: FileWildcard(InFileWildcard)
		, FoundFiles(InFoundFiles)
	{}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (bIsDirectory == false)
		{
			FString Filename(FilenameOrDirectory);
			if (Filename.MatchesWildcard(FileWildcard))
			{
				FoundFiles.Add(Filename);
			}
		}
		return true;
	}
};

class IBuildPatchServicesModule;

class FChunkSetupTask : public FRunnable, public IPlatformFile::FDirectoryVisitor
{
public:
	/** Input parameters */
	IBuildPatchServicesModule*	BPSModule;
	FString						InstallDir; // Intermediate directory where installed chunks may be waiting
	FString						ContentDir; // Directory where installed chunks need to live to be mounted
	FString						HoldingDir; // Directory where manifest for chunks that are out of date and can be use for updates but not mounted.
	const TArray<FString>*		CurrentMountPaks; // Array of already mounted Paks
	/** Output */
	FEvent*									CompleteEvent;
	TArray<FString>							MountedPaks;
	TMultiMap<uint32, IBuildManifestPtr>	InstalledChunks;
	TMultiMap<uint32, IBuildManifestPtr>	HoldingChunks;
	/** Working */
	TArray<FString>		FoundPaks;
	TArray<FString>		FoundManifests;
	FFileSearchVisitor	PakVisitor;
	FFileSearchVisitor	ManifestVisitor;
	TArray<FString>		ManifestsToRemove;
	uint32				Pass;

	FChunkSetupTask()
		: BPSModule(nullptr)
		, CurrentMountPaks(nullptr)
		, PakVisitor(TEXT("*.pak"), FoundPaks)
		, ManifestVisitor(TEXT("*.manifest"), FoundManifests)
	{
		CompleteEvent = FPlatformProcess::GetSynchEventFromPool(true);
	}

	virtual ~FChunkSetupTask()
	{
		FPlatformProcess::ReturnSynchEventToPool(CompleteEvent);
	}

	void SetupWork(IBuildPatchServicesModule* InBPSModule, FString InInstallDir, FString InContentDir, FString InHoldingDir, const TArray<FString>& InCurrentMountedPaks)
	{
		BPSModule = InBPSModule;
		InstallDir = InInstallDir;
		ContentDir = InContentDir;
		HoldingDir = InHoldingDir;
		CurrentMountPaks = &InCurrentMountedPaks;

		Pass = 0;
		InstalledChunks.Reset();
		MountedPaks.Reset();
		FoundManifests.Reset();
		FoundPaks.Reset();
		ManifestsToRemove.Reset();

		CompleteEvent->Reset();
	}

	void DoWork()
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.IterateDirectory(*InstallDir, *this);
		for (const auto& ToRemove : ManifestsToRemove)
		{
			PlatformFile.DeleteFile(*ToRemove);
		}
		++Pass;
		PlatformFile.IterateDirectory(*ContentDir, *this);
		++Pass;
		PlatformFile.IterateDirectory(*HoldingDir, *this);
		CompleteEvent->Trigger();
	}

	bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			return true;
		}
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		FoundManifests.Reset();
		PlatformFile.IterateDirectory(FilenameOrDirectory, ManifestVisitor);
		if (FoundManifests.Num() == 0)
		{
			return true;
		}
		
		// Only allow one manifest per folder, any more suggests corruption so mark the folder for delete
		if (FoundManifests.Num() > 1)
		{
			ManifestsToRemove.Append(FoundManifests);
			return true;
		}
		// Load the manifest, so that can be classed as installed
		auto Manifest = BPSModule->LoadManifestFromFile(FoundManifests[0]);
		if (!Manifest.IsValid())
		{
			//Something is wrong, suggests corruption so mark the folder for delete
			ManifestsToRemove.Append(FoundManifests);
			return true;
		}
		auto ChunkIDField = Manifest->GetCustomField(TEXT("ChunkID"));
		if (!ChunkIDField.IsValid())
		{
			//Something is wrong, suggests corruption so mark the folder for delete
			ManifestsToRemove.Append(FoundManifests);
			return true;
		}
		auto ChunkPatchField = Manifest->GetCustomField(TEXT("bIsPatch"));
		bool bIsPatch = ChunkPatchField.IsValid() ? ChunkPatchField->AsString() == TEXT("true") : false;
		uint32 ChunkID = (uint32)ChunkIDField->AsInteger();

		if (Pass == 1)
		{
			InstalledChunks.AddUnique(ChunkID, Manifest);
		}
		else if (Pass == 0 && ContentDir != InstallDir)
		{
			FString ChunkFdrName = FString::Printf(TEXT("%s%d"), !bIsPatch ? TEXT("base") : TEXT("patch"), ChunkID);
			FString DestDir = *FPaths::Combine(*ContentDir, *ChunkFdrName);
			if (PlatformFile.DirectoryExists(*DestDir))
			{
				PlatformFile.DeleteDirectoryRecursively(*DestDir);
			}
			PlatformFile.CreateDirectoryTree(*DestDir);
			if (PlatformFile.CopyDirectoryTree(*DestDir, FilenameOrDirectory, true))
			{
				ManifestsToRemove.Add(FilenameOrDirectory);
			}
		}
		else if (Pass == 2)
		{
			HoldingChunks.AddUnique(ChunkID, Manifest);
		}

		return true;
	}

	static const TCHAR *Name()
	{
		return TEXT("FChunkSetup");
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChunkSetupTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	uint32 Run()
	{
		DoWork();
		return 0;
	}

	bool IsDone()
	{
		return CompleteEvent->Wait(FTimespan::Zero());
	}
};

class FChunkMountTask : public FRunnable, public IPlatformFile::FDirectoryVisitor
{
public:
	/** Input parameters */
	IBuildPatchServicesModule*	BPSModule;
	FString						ContentDir; // Directory where installed chunks need to live to be mounted
	const TArray<FString>*		CurrentMountPaks; // Array of already mounted Paks
	const TSet<uint32>*			ExpectedChunks; //Manifests expected to be seen. Chunks not in this list are deleted
	/** Output */
	FEvent*									CompleteEvent;
	TArray<FString>							MountedPaks;
	/** Working */
	TArray<FString>		FoundPaks;
	TArray<FString>		FoundManifests;
	TArray<FString>		ChunkInstallToDestroy;
	FFileSearchVisitor	PakVisitor;
	FFileSearchVisitor	ManifestVisitor;

	FChunkMountTask()
		: BPSModule(nullptr)
		, CurrentMountPaks(nullptr)
		, PakVisitor(TEXT("*.pak"), FoundPaks)
		, ManifestVisitor(TEXT("*.manifest"), FoundManifests)
	{
		CompleteEvent = FPlatformProcess::GetSynchEventFromPool(true);
	}

	~FChunkMountTask()
	{
		FPlatformProcess::ReturnSynchEventToPool(CompleteEvent);
	}

	void SetupWork(IBuildPatchServicesModule* InBPSModule, FString InContentDir, const TArray<FString>& InCurrentMountedPaks, const TSet<uint32>& InExpectedChunks)
	{
		BPSModule = InBPSModule;
		ContentDir = InContentDir;
		CurrentMountPaks = &InCurrentMountedPaks;
		ExpectedChunks = &InExpectedChunks;

		MountedPaks.Reset();
		FoundManifests.Reset();
		FoundPaks.Reset();
		ChunkInstallToDestroy.Reset();

		CompleteEvent->Reset();
	}

	void DoWork()
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.IterateDirectory(*ContentDir, *this);
		for (const auto& Dir : ChunkInstallToDestroy)
		{
			PlatformFile.DeleteDirectoryRecursively(*Dir);
		}
		CompleteEvent->Trigger();
	}

	bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			return true;
		}
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		FoundManifests.Reset();
		PlatformFile.IterateDirectory(FilenameOrDirectory, ManifestVisitor);
		if (FoundManifests.Num() == 0)
		{
			return true;
		}
		// Only allow one manifest per folder, any more suggests corruption so ignore
		if (FoundManifests.Num() > 1)
		{
			return true;
		}
		// Load the manifest, so that can be classed as installed
		auto Manifest = BPSModule->LoadManifestFromFile(FoundManifests[0]);
		if (!Manifest.IsValid())
		{
			//Something is wrong, suggests corruption so ignore
			return true;
		}
		auto ChunkIDField = Manifest->GetCustomField(TEXT("ChunkID"));
		if (!ChunkIDField.IsValid())
		{
			//Something is wrong, suggests corruption so ignore
			return true;
		}

		if (!ExpectedChunks->Find(ChunkIDField->AsInteger())) 
		{
			//Add this to the list of chunks to remove
			ChunkInstallToDestroy.Add(FilenameOrDirectory);
			return true;
		}

		FoundPaks.Reset();
		PlatformFile.IterateDirectoryRecursively(FilenameOrDirectory, PakVisitor);
		if (FoundPaks.Num() == 0)
		{
			return true;
		}
		auto PakReadOrderField = Manifest->GetCustomField(TEXT("PakReadOrdering"));
		uint32 PakReadOrder = PakReadOrderField.IsValid() ? (uint32)PakReadOrderField->AsInteger() : 0;
		for (const auto& PakPath : FoundPaks)
		{
			//Note: A side effect here is that any unmounted paks get mounted. This is desirable as
			// any previously installed chunks get mounted 
			if (!CurrentMountPaks->Contains(PakPath) && !MountedPaks.Contains(PakPath))
			{
				if (FCoreDelegates::MountPak.IsBound())
				{
					auto bSuccess = FCoreDelegates::MountPak.Execute(PakPath, PakReadOrder);
#if !UE_BUILD_SHIPPING
					if (!bSuccess)
					{
						// This can fail because of the sandbox system - which the pak system doesn't understand.
						auto SandboxedPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*PakPath);
						bSuccess = FCoreDelegates::MountPak.Execute(SandboxedPath, PakReadOrder);
					}
#endif
					//Register the install
					BPSModule->RegisterAppInstallation(Manifest.ToSharedRef(), FilenameOrDirectory);
				}
			}
		}
		return true;
	}

	static const TCHAR *Name()
	{
		return TEXT("FChunkSetup");
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChunkMountTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	uint32 Run()
	{
		DoWork();
		return 0;
	}

	bool IsDone()
	{
		return CompleteEvent->Wait(FTimespan::Zero());
	}
};