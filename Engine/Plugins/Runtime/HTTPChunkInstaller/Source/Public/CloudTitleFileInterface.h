// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CloudDelegateMacros.h"

/**
* Paging info needed for a request that can return paged results
*/
class FCloudPagedQuery
{
public:
	FCloudPagedQuery(int32 InStart = 0, int32 InCount = -1)
		: Start(InStart)
		, Count(InCount)
	{}

	/** @return true if valid range */
	bool IsValidRange() const
	{
		return Start >= 0 && Count >= 0;
	}

	/** first entry to fetch */
	int32 Start;
	/** total entries to fetch. -1 means ALL */
	int32 Count;
};

/** Holds metadata about a given downloadable file */
struct FCloudHeader
{
	/** Hash value, if applicable, of the given file contents */
	FString Hash;
	/** The hash algorithm used to sign this file */
	FName HashType;
	/** Filename as downloaded */
	FString DLName;
	/** Logical filename, maps to the downloaded filename */
	FString FileName;
	/** File size */
	int32 FileSize;
	/** The full URL to download the file if it is stored in a CDN or separate host site */
	FString URL;
	/** The chunk id this file represents */
	uint32 ChunkID;

	/** Constructors */
	FCloudHeader() :
		FileSize(0),
		ChunkID(0)
	{}

	FCloudHeader(const FString& InFileName, const FString& InDLName, int32 InFileSize) :
		DLName(InDLName),
		FileName(InFileName),
		FileSize(InFileSize),
		ChunkID(0)
	{}

	bool operator==(const FCloudHeader& Other) const
	{
		return FileSize == Other.FileSize &&
			Hash == Other.Hash &&
			HashType == Other.HashType &&
			DLName == Other.DLName &&
			FileName == Other.FileName &&
			URL == Other.URL &&
			ChunkID == Other.ChunkID;
	}

	bool operator<(const FCloudHeader& Other) const
	{
		return FileName.Compare(Other.FileName, ESearchCase::IgnoreCase) < 0;
	}
};

/** The state of an async task (read friends, read content, write cloud file, etc) request */
namespace ECloudAsyncTaskState
{
	enum Type
	{
		/** The task has not been started */
		NotStarted,
		/** The task is currently being processed */
		InProgress,
		/** The task has completed successfully */
		Done,
		/** The task failed to complete */
		Failed
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(ECloudAsyncTaskState::Type EnumVal)
	{
		switch (EnumVal)
		{
		case NotStarted:
		{
			return TEXT("NotStarted");
		}
		case InProgress:
		{
			return TEXT("InProgress");
		}
		case Done:
		{
			return TEXT("Done");
		}
		case Failed:
		{
			return TEXT("Failed");
		}
		}
		return TEXT("");
	}
};

/** Holds the data used in downloading a file asynchronously from the online service */
struct FCloudEntry
{
	/** The name of the file as requested */
	FString FileName;
	/** The async state the file download is in */
	ECloudAsyncTaskState::Type AsyncState;
	/** The buffer of data for the file */
	TArray<uint8> Data;

	/** Constructors */
	FCloudEntry() :
		AsyncState(ECloudAsyncTaskState::NotStarted)
	{
	}

	FCloudEntry(const FString& InFileName) :
		FileName(InFileName),
		AsyncState(ECloudAsyncTaskState::NotStarted)
	{
	}

	virtual ~FCloudEntry() {}
};

/**
 * Delegate fired when the list of files has been returned from the network store
 *
 * @param bWasSuccessful whether the file list was successful or not
 *
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnEnumerateFilesComplete, bool);
typedef FOnEnumerateFilesComplete::FDelegate FOnEnumerateFilesCompleteDelegate;

/**
 * Delegate fired when as file read from the network platform's storage progresses
 *
 * @param FileName the name of the file this was for
 * @param NumBytes the number of bytes read so far
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadFileProgress, const FString&, uint64);
typedef FOnReadFileProgress::FDelegate FOnReadFileProgressDelegate;

/**
 * Delegate fired when a file read from the network platform's storage is complete
 *
 * @param bWasSuccessful whether the file read was successful or not
 * @param FileName the name of the file this was for
 *
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReadFileComplete, bool, const FString&);
typedef FOnReadFileComplete::FDelegate FOnReadFileCompleteDelegate;

class ICloudTitleFile
{
protected:
	ICloudTitleFile() {};

public:
	virtual ~ICloudTitleFile() {};

	/**
	 * Copies the file data into the specified buffer for the specified file
	 *
	 * @param FileName the name of the file to read
	 * @param FileContents the out buffer to copy the data into
	 *
 	 * @return true if the data was copied, false otherwise
	 */
	virtual bool GetFileContents(const FString& FileName, TArray<uint8>& FileContents) = 0;

	/**
	 * Empties the set of downloaded files if possible (no async tasks outstanding)
	 *
	 * @return true if they could be deleted, false if they could not
	 */
	virtual bool ClearFiles() = 0;

	/**
	 * Empties the cached data for this file if it is not being downloaded currently
	 *
	 * @param FileName the name of the file to remove from the cache
	 *
	 * @return true if it could be deleted, false if it could not
	 */
	virtual bool ClearFile(const FString& FileName) = 0;

	/**
	 * Delete cached files on disk
	 *
	 * @param bSkipEnumerated if true then only non-enumerated files are deleted
	 */
	virtual void DeleteCachedFiles(bool bSkipEnumerated) = 0;

	/**
	* Requests a list of available files from the network store
	*
	* @param Page paging info to use for query
	*
	* @return true if the request has started, false if not
	*/
	virtual bool EnumerateFiles(const FCloudPagedQuery& Page = FCloudPagedQuery()) = 0;

	/**
	 * Delegate fired when the list of files has been returned from the network store
	 *
	 * @param bWasSuccessful whether the file list was successful or not
	 *
	 */
	DEFINE_CLOUD_DELEGATE_ONE_PARAM(OnEnumerateFilesComplete, bool);

	/**
	 * Returns the list of files that was returned by the network store
	 * 
	 * @param Files out array of file metadata
	 *
	 */
	virtual void GetFileList(TArray<FCloudHeader>& Files) = 0;

	/**
	 * Starts an asynchronous read of the specified file from the network platform's file store
	 *
	 * @param FileToRead the name of the file to read
	 *
	 * @return true if the calls starts successfully, false otherwise
	 */
	virtual bool ReadFile(const FString& FileName) = 0;

	/**
	 * Delegate fired when a file read from the network platform's storage is complete
	 *
	 * @param bWasSuccessful whether the file read was successful or not
	 * @param FileName the name of the file this was for
	 *
	 */
	DEFINE_CLOUD_DELEGATE_TWO_PARAM(OnReadFileComplete, bool, const FString&);

	/**
	 * Delegate fired when as file read from the network platform's storage progresses
	 *
	 * @param FileName the name of the file this was for
	 * @param NumBytes the number of bytes read so far
	 */
	DEFINE_CLOUD_DELEGATE_TWO_PARAM(OnReadFileProgress, const FString&, uint64);
};

typedef TSharedPtr<ICloudTitleFile, ESPMode::ThreadSafe> ICloudTitleFilePtr;
typedef TSharedRef<ICloudTitleFile, ESPMode::ThreadSafe> ICloudTitleFileRef;
