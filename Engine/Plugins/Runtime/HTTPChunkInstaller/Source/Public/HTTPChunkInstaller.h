// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Templates/UniquePtr.h"
#include "CloudTitleFileInterface.h"
#include "ChunkInstall.h"
#include "ChunkSetup.h"
#include "Containers/Ticker.h"

/**
* HTTP based implementation of chunk based install
**/
class HTTPCHUNKINSTALLER_API FHTTPChunkInstall : public FGenericPlatformChunkInstall, public FTSTickerObjectBase
{
public:
	FHTTPChunkInstall();
	~FHTTPChunkInstall();

	virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) override
	{
		if (ReportType == EChunkProgressReportingType::PercentageComplete)
		{
			return true;
		}

		return false;
	}

	virtual float GetChunkProgress( uint32 ChunkID, EChunkProgressReportingType::Type ReportType ) override;

	virtual EChunkInstallSpeed::Type GetInstallSpeed() override
	{
		return InstallSpeed;
	}

	virtual bool SetInstallSpeed( EChunkInstallSpeed::Type InInstallSpeed ) override
	{
		if (InstallSpeed != InInstallSpeed)
		{
			InstallSpeed = InInstallSpeed;
			if (InstallService.IsValid() && 
				((InstallSpeed == EChunkInstallSpeed::Paused && !InstallService->IsPaused()) || (InstallSpeed != EChunkInstallSpeed::Paused && InstallService->IsPaused())))
			{
				InstallService->TogglePauseInstall();
			}
		}
		return true;
	}
	
	virtual bool PrioritizeChunk( uint32 ChunkID, EChunkPriority::Type Priority ) override;

	virtual bool DebugStartNextChunk()
	{
		/** Unless paused we are always installing! */
		InstallerState = ChunkInstallState::ReadTitleFiles;
		return false;
	}

	bool Tick(float DeltaSeconds) override;

	void EndInstall();

private:

	void InitialiseSystem();
	bool AddDataToFileCache(const FString& ManifestHash,const TArray<uint8>& Data);
	bool IsDataInFileCache(const FString& ManifestHash);
	bool GetDataFromFileCache(const FString& ManifestHash,TArray<uint8>& Data);
	bool RemoveDataFromFileCache(const FString& ManifestHash);
	void UpdatePendingInstallQueue();
	IBuildManifestPtr FindPreviousInstallManifest(const IBuildManifestPtr& ChunkManifest);
	void BeginChunkInstall(uint32 NextChunkID,IBuildManifestPtr ChunkManifest, IBuildManifestPtr PrevInstallChunkManifest);
	void ParseTitleFileManifest(const FString& ManifestFileHash);
	bool BuildChunkFolderName(IBuildManifestRef Manifest, FString& ChunkFdrName, FString& ManifestName, uint32& ChunkID, bool& bIsPatch);
	void OSSEnumerateFilesComplete(bool bSuccess);
	void OSSReadFileComplete(bool bSuccess, const FString& Filename);
	void OSSInstallComplete(const IBuildInstallerRef& Installer);

	virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) override;

	enum struct ChunkInstallState
	{
		Setup,
		SetupWait,
		QueryRemoteManifests,
		EnterOfflineMode,
		MoveInstalledChunks,
		RequestingTitleFiles,
		SearchTitleFiles,
		ReadTitleFiles,
		WaitingOnRead,
		ReadComplete,
		PostSetup,
		Idle,
		Installing,
		CopyToContent,
	};

	struct FChunkPrio
	{
		FChunkPrio(uint32 InChunkID, EChunkPriority::Type InChunkPrio)
			: ChunkID(InChunkID)
			, ChunkPrio(InChunkPrio)
		{

		}

		uint32					ChunkID;
		EChunkPriority::Type	ChunkPrio;

		bool operator == (const FChunkPrio& RHS) const
		{
			return ChunkID == RHS.ChunkID;
		}

		bool operator < (const FChunkPrio& RHS) const
		{
			return ChunkPrio < RHS.ChunkPrio;
		}
	};

	FChunkInstallTask											ChunkCopyInstall;
	TUniquePtr<FRunnableThread>									ChunkCopyInstallThread;
	FChunkSetupTask												ChunkSetupTask;
	TUniquePtr<FRunnableThread>									ChunkSetupTaskThread;
	FChunkMountTask												ChunkMountTask;
	TUniquePtr<FRunnableThread>									ChunkMountTaskThread;
	TMultiMap<uint32, IBuildManifestPtr>						InstalledManifests;
	TMultiMap<uint32, IBuildManifestPtr>						PrevInstallManifests;
	TMultiMap<uint32, IBuildManifestPtr>						RemoteManifests;
	TSet<FString>												ManifestsInMemory;
	TSet<uint32>												ExpectedChunks;
	TArray<FCloudHeader>										TitleFilesToRead;
	TArray<uint8>												FileContentBuffer;
	TArray<FChunkPrio>											PriorityQueue;
	TArray<FString>												MountedPaks;
	ICloudTitleFilePtr											OnlineTitleFile;
	ICloudTitleFilePtr											OnlineTitleFileHttp;
	IBuildInstallerPtr											InstallService;
	IBuildManifestPtr											InstallingChunkManifest;
	FDelegateHandle												EnumFilesCompleteHandle;
	FDelegateHandle												ReadFileCompleteHandle;
	FString														CloudDir;
	FString														CloudDirectory;
	FString														StageDir;
	FString														InstallDir;
	FString														BackupDir;
	FString														ContentDir;
	FString														CacheDir;
	FString														HoldingDir;
	IBuildPatchServicesModule*									BPSModule;
	uint32														InstallingChunkID;
	ChunkInstallState											InstallerState;
	EChunkInstallSpeed::Type									InstallSpeed;
	bool														bFirstRun;
	bool														bSystemInitialised;
#if !UE_BUILD_SHIPPING
	bool														bDebugNoInstalledRequired;
#endif
};
