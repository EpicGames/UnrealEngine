// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CloudTitleFileInterface.h"

class FLocalTitleFile : public ICloudTitleFile
{
public:

	FLocalTitleFile(const FString& InRootDirectory);

	// ICloudTitleFile interface

	virtual bool GetFileContents(const FString& DLName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles() override;
	virtual bool ClearFile(const FString& DLName) override;
	virtual void DeleteCachedFiles(bool bSkipEnumerated) override;
	virtual bool EnumerateFiles(const FCloudPagedQuery& Page = FCloudPagedQuery()) override;
	virtual void GetFileList(TArray<FCloudHeader>& InFileHeaders) override;
	virtual bool ReadFile(const FString& DLName) override;

protected:

	FString GetFileNameFromDLName(const FString& DLName) const;

private:

	FString							RootDirectory;
	TArray<FCloudHeader>			FileHeaders;
	TMap< FString, TArray<uint8> >	DLNameToFileContents;
};